// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AndroidLocalNotification : ModuleRules
{
	public AndroidLocalNotification(ReadOnlyTargetRules Target) : base(Target)
	{
		BinariesSubFolder = "Android";

		PublicIncludePathModuleNames.AddRange(new string[]
		{
			"AndroidLocalNotification",
			"Engine",
		});

        PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
            "Launch"
		});
	}
}
