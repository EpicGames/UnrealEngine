// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJniGameActivity.h"

namespace UE::Jni
{
	struct FLocalNotification
	{
		using PartialClass = FGameActivity;
		
		static void JNICALL nativeAppOpenedWithLocalNotification(JNIEnv* env, jobject thiz, jstring jactivationEvent, int32 jFireDate);
		
		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(nativeAppOpenedWithLocalNotification)
		};
	};
	
	template struct TInitialize<FLocalNotification>;
}
#endif
