// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 	AndroidLocalNotification.cpp: Unreal AndroidLocalNotification service interface object.
 =============================================================================*/

/*------------------------------------------------------------------------------------
	Includes
 ------------------------------------------------------------------------------------*/

#include "AndroidLocalNotification.h"
#include "AndroidJniLocalNotification.h"
#include "Async/TaskGraphInterfaces.h"
#include "Stats/Stats.h"

DEFINE_LOG_CATEGORY(LogAndroidLocalNotification);

class FAndroidLocalNotificationModule : public ILocalNotificationModule
{
public:
	/** Creates a new instance of the service implemented by the module. */
	virtual ILocalNotificationService* GetLocalNotificationService() override
	{
		static ILocalNotificationService*	oneTrueLocalNotificationService = nullptr;
		
		if(oneTrueLocalNotificationService == nullptr)
		{
			oneTrueLocalNotificationService = new FAndroidLocalNotificationService;
		}
		
		return oneTrueLocalNotificationService;
	}
};


#if USE_ANDROID_JNI
void JNICALL UE::Jni::FLocalNotification::nativeAppOpenedWithLocalNotification(JNIEnv* jenv, jobject thiz, jstring jactivationEvent, int32 jFireDate)
{
	auto ActivationEvent = FJavaHelper::FStringFromParam(jenv, jactivationEvent);
	
	int32 FireDate = (int32)jFireDate;

	DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.ProcessAppOpenedWithLocalNotification"), STAT_FSimpleDelegateGraphTask_ProcessAppOpenedWithLocalNotification, STATGROUP_TaskGraphTasks);

	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
		FSimpleDelegateGraphTask::FDelegate::CreateLambda([=]() 
		{
			ILocalNotificationService* AndroidLocalNotificationService = NULL;

			FString ModuleName = "AndroidLocalNotification";
			// load the module by name
			ILocalNotificationModule* module = FModuleManager::LoadModulePtr<ILocalNotificationModule>(*ModuleName);
			// did the module exist?
			if (module != nullptr)
			{
				AndroidLocalNotificationService = module->GetLocalNotificationService();
				AndroidLocalNotificationService->SetLaunchNotification(ActivationEvent, FireDate);
			}
		}),
		GET_STATID(STAT_FSimpleDelegateGraphTask_ProcessAppOpenedWithLocalNotification),
		nullptr,
		ENamedThreads::GameThread
		);
}
#endif


IMPLEMENT_MODULE(FAndroidLocalNotificationModule, AndroidLocalNotification);

/*------------------------------------------------------------------------------------
	FAndroidLocalNotification
 ------------------------------------------------------------------------------------*/
FAndroidLocalNotificationService::FAndroidLocalNotificationService(){
	AppLaunchedWithNotification = false;
	LaunchNotificationActivationEvent = "None";
	LaunchNotificationFireDate = 0;
}

void FAndroidLocalNotificationService::ClearAllLocalNotifications()
{
#if USE_ANDROID_JNI
	extern void AndroidThunkCpp_ClearAllLocalNotifications();
	AndroidThunkCpp_ClearAllLocalNotifications();
#endif
}

int32 FAndroidLocalNotificationService::ScheduleLocalNotificationAtTime(const FDateTime& FireDateTime, bool LocalTime, const FText& Title, const FText& Body, const FText& Action, const FString& ActivationEvent)
{
#if USE_ANDROID_JNI
	extern int32 AndroidThunkCpp_ScheduleLocalNotificationAtTime(const FDateTime & FireDateTime, bool LocalTime, const FText & Title, const FText & Body, const FText & Action, const FString & ActivationEvent, int32 IdOverride);
	return AndroidThunkCpp_ScheduleLocalNotificationAtTime(FireDateTime, LocalTime, Title, Body, Action, ActivationEvent, -1);
#else
	return -1;
#endif
}

int32 FAndroidLocalNotificationService::ScheduleLocalNotificationAtTimeOverrideId(const FDateTime& FireDateTime, bool LocalTime, const FText& Title, const FText& Body, const FText& Action, const FString& ActivationEvent, int32 IdOverride)
{
#if USE_ANDROID_JNI
	extern int32 AndroidThunkCpp_ScheduleLocalNotificationAtTime(const FDateTime & FireDateTime, bool LocalTime, const FText & Title, const FText & Body, const FText & Action, const FString & ActivationEvent, int32 IdOverride);
	return AndroidThunkCpp_ScheduleLocalNotificationAtTime(FireDateTime, LocalTime, Title, Body, Action, ActivationEvent, IdOverride);
#else
	return -1;
#endif
}

int32 FAndroidLocalNotificationService::ScheduleLocalNotificationBadgeAtTime(const FDateTime& FireDateTime, bool LocalTime, const FString& ActivationEvent)
{
	// Do nothing...
	return -1;
}

void FAndroidLocalNotificationService::GetLaunchNotification(bool& NotificationLaunchedApp, FString& ActivationEvent, int32& FireDate)
{
#if USE_ANDROID_JNI
	extern void AndroidThunkCpp_GetLaunchNotification(bool& NotificationLaunchedApp, FString& ActivationEvent, int32& FireDate);
	AndroidThunkCpp_GetLaunchNotification(NotificationLaunchedApp, ActivationEvent, FireDate);
#endif
}

void FAndroidLocalNotificationService::SetLaunchNotification(FString const& ActivationEvent, int32 FireDate)
{
	// Don't do anything here since this is taken care of in java land
}

void FAndroidLocalNotificationService::CancelLocalNotification(const FString& ActivationEvent)
{
	// TODO
}

void FAndroidLocalNotificationService::CancelLocalNotification(int32 NotificationId)
{
	if (NotificationId < 0)
	{
		return;
	}
	
#if USE_ANDROID_JNI
	extern bool AndroidThunkCpp_DestroyScheduledNotificationIfExists(int32 NotificationId);
	AndroidThunkCpp_DestroyScheduledNotificationIfExists(NotificationId);
#endif
}
