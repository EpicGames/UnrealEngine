// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidLocalNotification.h: Unreal Android local notification interface object.
 =============================================================================*/

#pragma once

#include "LocalNotification.h"

#if PLATFORM_ANDROID
	#include "Android/AndroidApplication.h"
	#include <android_native_app_glue.h>
#endif

#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAndroidLocalNotification, Log, All);

/**
 * AndroidLocalNotification implementation of an Unreal local notification service.
 */
class FAndroidLocalNotificationService : public ILocalNotificationService
{
public:
	FAndroidLocalNotificationService();

	virtual void ClearAllLocalNotifications();
	virtual int32 ScheduleLocalNotificationAtTime(const FDateTime& FireDateTime, bool LocalTime, const FText& Title, const FText& Body, const FText& Action, const FString& ActivationEvent);
	virtual int32 ScheduleLocalNotificationAtTimeOverrideId(const FDateTime& FireDateTime, bool LocalTime, const FText& Title, const FText& Body, const FText& Action, const FString& ActivationEvent, int32 IdOverride);
	virtual int32 ScheduleLocalNotificationBadgeAtTime(const FDateTime& FireDateTime, bool LocalTime, const FString& ActivationEvent);
	virtual void GetLaunchNotification(bool& NotificationLaunchedApp, FString& ActivationEvent, int32& FireDate);
	virtual void SetLaunchNotification(FString const& ActivationEvent, int32 FireDate);
	virtual void CancelLocalNotification(const FString& ActivationEvent);
	virtual void CancelLocalNotification(int NotificationId);

private:
	bool	AppLaunchedWithNotification;
	FString	LaunchNotificationActivationEvent;
	int32	LaunchNotificationFireDate;
};
