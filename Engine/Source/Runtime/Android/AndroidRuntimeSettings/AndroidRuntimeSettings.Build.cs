// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AndroidRuntimeSettings : ModuleRules
{
	public AndroidRuntimeSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		BinariesSubFolder = "Android";

        PublicDependencyModuleNames.Add("AudioPlatformConfiguration");

        PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
                "Engine"
            }
		);

        if (Target.Type == TargetType.Editor || Target.Type == TargetType.Program)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
			    {
                    "TargetPlatform"
			    }
            );

            PrivateIncludePathModuleNames.Add("AndroidTargetPlatform");
            PrivateIncludePathModuleNames.Add("AndroidTargetPlatformControls");
        }
	}
}
