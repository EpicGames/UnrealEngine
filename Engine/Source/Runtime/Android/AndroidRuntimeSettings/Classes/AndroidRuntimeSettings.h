// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "AudioCompressionSettings.h"

#include "AndroidRuntimeSettings.generated.h"

#define UE_API ANDROIDRUNTIMESETTINGS_API

DECLARE_LOG_CATEGORY_EXTERN(LogAndroidRuntimeSettings, Log, All);

UENUM()
namespace EAndroidScreenOrientation
{
	// IF THIS CHANGES, MAKE SURE TO UPDATE UEDeployAndroid.cs, ConvertOrientationIniValue()!

	enum Type : int
	{
		/** Portrait orientation (the display is taller than it is wide). */
		Portrait,

		/** Portrait orientation rotated 180 degrees. */
		ReversePortrait,

		/** Use either portrait or reverse portrait orientation, where supported by the device, based on the device orientation sensor. */
		SensorPortrait,

		/** Landscape orientation (the display is wider than it is tall). */
		Landscape,

		/** Landscape orientation rotated 180 degrees. */
		ReverseLandscape,

		/** Use either landscape or reverse landscape orientation, based on the device orientation sensor. */
		SensorLandscape,

		/** Use any orientation the device normally supports, based on the device orientation sensor. */
		Sensor,

		/** Use any orientation (including ones the device wouldn't choose in Sensor mode), based on the device orientation sensor. */
		FullSensor,
	};
}

/** Rounded edge safe zone preferences */
UENUM()
namespace EAndroidRoundedEdgeSafeZoneDirection
{
	enum Type : int
	{
		None,
		Horizontal,
		Vertical,
		Both
	};
}

/** Depth buffer precision preferences */
UENUM()
namespace EAndroidDepthBufferPreference
{
	// IF THIS CHANGES, MAKE SURE TO UPDATE UEDeployAndroid.cs, ConvertDepthBufferIniValue()!

	enum Type : int
	{
		Default = 0 UMETA(DisplayName = "Default"),
		Bits16 = 16 UMETA(DisplayName = "16-bit"),
		Bits24 = 24 UMETA(DisplayName = "24-bit"),
		Bits32 = 32 UMETA(DisplayName = "32-bit"),
	};
}

/** The default install location for the application */
UENUM()
namespace EAndroidInstallLocation
{
	enum Type : int
	{
		/** Install your app only on internal device storage */
		InternalOnly,
		/** Install your app on external storage when available */
		PreferExternal,
		/** Internal storage is preferred over external, unless the interal storage is low on space */
		Auto
	};
}

/** The target Oculus Mobile device for application packaging */
UENUM(meta = (Deprecated = "5.8", DeprecationMessage = "EOculusMobileDevice is no longer used."))
namespace EOculusMobileDevice
{
	enum Type : int
	{
		// 0 was the deprecated OculusGo

		/** Package for Oculus Quest */
		Quest = 1 UMETA(DisplayName = "Oculus Quest"),

		/** Package for Oculus Quest 2*/
		Quest2 = 2 UMETA(DisplayName = "Oculus Quest 2"),
	};
}

/** AdMob TagForChildDirectedTreament  */
UENUM()
namespace ETagForChildDirectedTreatment
{
	enum Type : int
	{
		TAG_FOR_CHILD_DIRECTED_TREATMENT_UNSPECIFIED = 0,
		TAG_FOR_CHILD_DIRECTED_TREATMENT_TRUE = 1,
		TAG_FOR_CHILD_DIRECTED_TREATMENT_FALSE = 2,
	};
}

/** AdMob TagForChildDirectedTreament  */
UENUM()
namespace ETagForUnderAgeOfConsent
{
	enum Type : int
	{
		TAG_FOR_UNDER_AGE_OF_CONSENT_UNSPECIFIED = 0,
		TAG_FOR_UNDER_AGE_OF_CONSENT_TRUE = 1,
		TAG_FOR_UNDER_AGE_OF_CONSENT_FALSE = 2,
	};
}

/** AdMob MaxAdContentRating  */
UENUM()
namespace EMaxAdContentRating
{
	enum Type : int
	{
		MAX_AD_CONTENT_RATING_G = 0,
		MAX_AD_CONTENT_RATING_PG = 1,
		MAX_AD_CONTENT_RATING_T = 2,
		MAX_AD_CONTENT_RATING_MA = 2,
	};
}

/**
 * Holds the game-specific achievement name and corresponding ID from Google Play services.
 */
USTRUCT()
struct FGooglePlayAchievementMapping
{
	GENERATED_USTRUCT_BODY()

	/** The game-specific achievement name (the one passed in to WriteAchievement calls). */
	UPROPERTY(EditAnywhere, Category = GooglePlayServices)
	FString Name;

	/** The ID of the corresponding achievement, generated by the Google Play developer console. */
	UPROPERTY(EditAnywhere, Category = GooglePlayServices)
	FString AchievementID;
};

/**
 * Holds the game-specific leaderboard name and corresponding ID from Google Play services.
 */
USTRUCT()
struct FGooglePlayLeaderboardMapping
{
	GENERATED_USTRUCT_BODY()

	/** The game-specific leaderboard name (the one passed in to WriteLeaderboards calls). */
	UPROPERTY(EditAnywhere, Category = GooglePlayServices)
	FString Name;

	/** The ID of the corresponding leaderboard, generated by the Google Play developer console. */
	UPROPERTY(EditAnywhere, Category = GooglePlayServices)
	FString LeaderboardID;
};

UENUM()
namespace EAndroidAudio
{
	enum UE_DEPRECATED(5.5, "Please don't use") Type : int
	{
		Default = 0 UMETA(DisplayName = "Default", ToolTip = "This option selects the default encoder."),
		OGG = 1 UMETA(DisplayName = "Ogg Vorbis", ToolTip = "Selects Ogg Vorbis encoding."),
		ADPCM = 2 UMETA(DisplayName = "ADPCM", ToolTip = "This option selects ADPCM lossless encoding.")
	};
}

UENUM()
namespace EAndroidGraphicsDebugger
{
	enum UE_DEPRECATED(5.8, "Please don't use") Type : int
	{
		None = 0 UMETA(DisplayName = "None"),
		Mali = 1 UMETA(DisplayName = "Mali Graphics Debugger", ToolTip = "Configure for Mali Graphics Debugger."),
		Adreno = 2 UMETA(DisplayName = "Adreno Profiler", ToolTip = "Configure for Adreno Profiler."),
	};
}


/**
 * Implements the settings for the Android runtime platform.
 */
UCLASS(MinimalAPI, config=Engine, defaultconfig)
class UAndroidRuntimeSettings : public UObject
{
public:
	GENERATED_UCLASS_BODY()

	// The official name of the product (same as the name you use on the Play Store web site). Note: Must have at least 2 sections separated by a period and be unique!
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Android Package Name ('com.Company.Project', [PROJECT] is replaced with project name)"))
	FString PackageName;

	// The version number used to indicate newer versions in the Store
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Store Version (1-2147483647)", ClampMin="1", ClampMax="2147483647"))
	int32 StoreVersion;

	// Offset to add to store version for APKs generated for arm64
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", meta = (DisplayName = "Store Version offset (arm64)"))
	int32 StoreVersionOffsetArm64;

	// Offset to add to store version for APKs generated for x86_64
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", meta = (DisplayName = "Store Version offset (x86_64)"))
	int32 StoreVersionOffsetX8664;

	// The visual application name displayed for end users
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Application Display Name (app_name), project name if blank"))
	FString ApplicationDisplayName;

	// The visual version displayed for end users
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Version Display Name (usually x.y)"))
	FString VersionDisplayName;

	// What OS version the app is allowed to be installed on (do not set this lower than 26)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Minimum SDK Version (26=8.0.0, 27=8.1.0, 28=9, 29=10, 30=11, 31=12)"))
	int32 MinSDKVersion;
	
	// What OS version the app is expected to run on (do not set this lower than 26)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Target SDK Version (26=8.0.0, 27=8.1.0, 28=9, 29=10, 30=11, 31=12)"))
	int32 TargetSDKVersion;

	// Preferred install location for the application
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging")
	TEnumAsByte<EAndroidInstallLocation::Type> InstallLocation;

	// Enable -Xlint:unchecked and -Xlint:depreciation for Java compiling (Gradle only)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Enable Lint depreciation checks"))
	bool bEnableLint;

	// Should the data be placed into the .apk file instead of a separate .obb file. Amazon requires this to be enabled, but Google Play Store will not allow .apk files larger than 100MB, so only small games will work with this enabled.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Package game data inside .apk?"))
	bool bPackageDataInsideApk;

	// If checked, both batch (.bat) files and shell script (.command) files will be generated, otherwise only done for the current system (default)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Generate install files for all platforms"))
	bool bCreateAllPlatformsInstall;

	// Disable the verification of an OBB file when it is downloaded or on first start when in a distribution build. 
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Disable verify OBB on first start/update."))
	bool bDisableVerifyOBBOnStartUp;

	// If checked, OBB is limited to 1 GiB.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Force small OBB files."))
	bool bForceSmallOBBFiles;

	// If checked, OBB is not limited to 2 GiB allowed by Google Play Store (still limited to 4 GiB ZIP limit). 
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Allow large OBB files."))
	bool bAllowLargeOBBFiles;

	// If checked, a patch OBB is generated for files not fitting in the main OBB (requires using multiple PAK files so split up content by chunk id).
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Allow patch OBB file."))
	bool bAllowPatchOBBFile;

	// If checked, up to two additional overflow OBB files are generated for files not fitting in the patch OBB (requires using multiple PAK files so split up content by chunk id).
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Allow overflow OBB files."))
	bool bAllowOverflowOBBFiles;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Don't bundle libraries into .apk for quicker iteration [Experimental]"))
	bool bDontBundleLibrariesInAPK;

	// If checked, UnrealGame files will be placed in ExternalFilesDir which is removed on uninstall.
	// You should also check this if you need to save you game progress without requesting runtime WRITE_EXTERNAL_STORAGE permission in android api 23+
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Use ExternalFilesDir for UnrealGame files?"))
	bool bUseExternalFilesDir;

	// If checked, log files will always be placed in a publicly available directory (either /sdcard/Android or /sdcard/UnrealGame).
	// You may require WRITE_EXTERNAL_STORAGE permission if you do not use ExternalFilesDir checkbox in android api 23+
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Make log files always publicly accessible?"))
	bool bPublicLogFiles;

	// The permitted orientation of the application on the device
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging")
	TEnumAsByte<EAndroidScreenOrientation::Type> Orientation;

	// Minimum supported aspect ratio (width / height). Android will automatically letterbox application on devices with smaller aspect ratio
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Minimum supported aspect ratio."))
	float MinAspectRatio;

	// Maximum supported aspect ratio (width / height). Android will automatically letterbox application on devices with bigger aspect ratio
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Maximum supported aspect ratio."))
	float MaxAspectRatio;

	// Enables use of the display cutout area on Android 9+
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Use display cutout region?"))
	bool bUseDisplayCutout;

	// Enables limiting safe zone away from rounded edges on Android 12+
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Limit safe zone to screen rounded edges."))
	TEnumAsByte<EAndroidRoundedEdgeSafeZoneDirection::Type> RoundedEdgeSafeZone;
	
	// Allow resizing of the window on Android devices with splitscreen
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Allow splitscreen resizing?"))
	bool bAllowResizing;

	// Allow support for size change when foldable and flip devices change screen or layout on Android 10+
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Allow fold/flip size changes?"))
	bool bSupportSizeChanges;

	// Should we restore scheduled local notifications on reboot? This will add a receiver for boot complete and a permission to the manifest.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Restore scheduled notifications on reboot"))
	bool bRestoreNotificationsOnReboot;

	// Should the software navigation buttons be hidden or not
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Enable FullScreen Immersive on KitKat and above devices."))
	bool bFullScreen;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = ( DisplayName = "Enable improved virtual keyboard"))
	bool bEnableNewKeyboard;
	
	// The preferred depth buffer bitcount for Android
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Preferred Depth Buffer format"))
	TEnumAsByte<EAndroidDepthBufferPreference::Type> DepthBufferPreference;

	// Verifies the device supports at least one of the cooked texture formats at runtime
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Validate texture formats"))
	bool bValidateTextureFormats;

	// When building for MinSDKVersion >= 23 gradle will leave native libs uncompressed in the apk. This flag might be helpful for builds that are not intended to be distributed via Google Play
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Force Gradle to compress native libs irregardless of MinSDKVersion setting"))
	bool bForceCompressNativeLibs;

	// Generates Android binary with RELR and APS2 relocation tables when building for MinSDKVersion >= 28 or just APS2 when building for MinSDKVersion >= 23
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "APK Packaging", Meta = (DisplayName = "Enable compression of relocation tables (and more). Depends on MinSDKVersion setting"))
	bool bEnableAdvancedBinaryCompression;

	// Enables generating AAB bundle
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "App Bundles", Meta = (DisplayName = "Generate bundle (AAB)"))
	bool bEnableBundle;

	// Enables generating universal APK from bundle
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "App Bundles", Meta = (DisplayName = "Generate universal APK from bundle", EditCondition = "bEnableBundle"))
	bool bEnableUniversalAPK;

	// Separate ABIs into their own APK in bundle
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "App Bundles", Meta = (DisplayName = "Enable ABI split", EditCondition = "bEnableBundle"))
	bool bBundleABISplit;

	// Separate resources by language into their own APK in bundle
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "App Bundles", Meta = (DisplayName = "Enable language split", EditCondition = "bEnableBundle"))
	bool bBundleLanguageSplit;

	// Separate resources by density into their own APK in bundle
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "App Bundles", Meta = (DisplayName = "Enable density split", EditCondition = "bEnableBundle"))
	bool bBundleDensitySplit;

	// Any extra tags for the <manifest> node
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Advanced APK Packaging", Meta = (DisplayName = "Extra Tags for <manifest> node"))
	TArray<FString> ExtraManifestNodeTags;

	// Any extra tags for the <application> node
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Advanced APK Packaging", Meta = (DisplayName = "Extra Tags for <application> node"))
	TArray<FString> ExtraApplicationNodeTags;

	// Any extra tags for the com.epicgames.unreal.GameActivity <activity> node
	// Any extra settings for the <application> section (an optional file <Project>/Build/Android/ManifestApplicationAdditions.txt will also be included)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Advanced APK Packaging", Meta = (DisplayName = "Extra Settings for <application> section (\\n to separate lines)"))
	FString ExtraApplicationSettings;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Advanced APK Packaging", Meta = (DisplayName = "Extra Tags for com.epicgames.unreal.GameActivity <activity> node"))
	TArray<FString> ExtraActivityNodeTags;

	// Any extra settings for the main <activity> section (an optional file <Project>/Build/Android/ManifestApplicationActivtyAdditions.txt will also be included)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Advanced APK Packaging", Meta = (DisplayName = "Extra Settings for <activity> section (\\n to separate lines)"))
	FString ExtraActivitySettings;

	// Any extra permissions your app needs (an optional file <Project>/Build/Android/ManifestRequirementsAdditions.txt will also be included,
	// or an optional file <Project>/Build/Android/ManifestRequirementsOverride.txt will replace the entire <!-- Requirements --> section)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Advanced APK Packaging", Meta = (DisplayName = "Extra Permissions (e.g. 'android.permission.INTERNET')"))
	TArray<FString> ExtraPermissions;

	// Add required permission to support Voice chat
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Advanced APK Packaging", Meta = (DisplayName = "Add permissions to support Voice chat (RECORD_AUDIO)"))
	bool bAndroidVoiceEnabled;

	// Request permission at startup (SplashActivity) for Voice chat (RECORD_AUDIO)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Advanced APK Packaging", Meta = (DisplayName = "Request permission at startup for Voice chat (RECORD_AUDIO)", EditCondition = "bAndroidVoiceEnabled"))
	bool bRecordPermissionAtStartupEnabled;

	// Add required permission and support to allow multicast/broadcast Wi-Fi traffic through network interface
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Advanced APK Packaging", Meta = (DisplayName = "Add support for multicast Wi-Fi traffic (CHANGE_WIFI_MULTICAST_STATE)"))
	bool bEnableMulticastSupport;

	// Package for Android OpenXR. This will result in changes to the android manifest and some runtime behavior changes related to XR applications.  Do not query this setting at runtime in a game, use AndroidThunkCpp_IsXRImmersiveHMDApplication() or AndroidThunkCpp_IsOpenXRImmersiveHMDApplication(). 
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Advanced APK Packaging", Meta = (DisplayName = "Package for OpenXR Immersive devices in general."))
	bool bPackageForOpenXRImmersive;
	
	// Package for Oculus Mobile devices. When enabled, it will enable build support for arm64 and vulkan, and disable build support for x86_64, Vulkan Desktop, and OpenGL.  Also manifest changes and some runtime behavior changes.  Do not query this setting at runtime in a game, use AndroidThunkCpp_IsXRImmersiveHMDApplication() or AndroidThunkCpp_IsOculusMobileApplication().
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Advanced APK Packaging", Meta = (DisplayName = "Package for Meta Quest devices."))
	bool bPackageForMetaQuest;

	// Removes Oculus Signature Files (osig) from APK if Quest/Go APK signed for distribution and enables entitlement checker
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Advanced APK Packaging", Meta = (DisplayName = "Remove Oculus Signature Files from Distribution APK"))
	bool bRemoveOSIG;

	// This is the file that keytool outputs, specified with the -keystore parameter (file should be in <Project>/Build/Android)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = DistributionSigning, Meta = (DisplayName = "Key Store (output of keytool, placed in <Project>/Build/Android)"))
	FString KeyStore;
	
	// This is the name of the key that you specified with the -alias parameter to keytool
	UPROPERTY(GlobalConfig, EditAnywhere, Category = DistributionSigning, Meta = (DisplayName = "Key Alias (-alias parameter to keytool)"))
	FString KeyAlias;
	
	// This is the password that you specified FOR THE KEYSTORE NOT THE KEY, when running keytool (either with -storepass or by typing it in).
	UPROPERTY(GlobalConfig, EditAnywhere, Category = DistributionSigning, Meta = (DisplayName = "Key Store Password (-storepass parameter to keytool)"))
	FString KeyStorePassword;
	
	// This is the password for the key that you may have specified with keytool, if it's different from the keystore password. Leave blank to use same as Keystore
	UPROPERTY(GlobalConfig, EditAnywhere, Category = DistributionSigning, Meta = (DisplayName = "Key Password (leave blank to use Key Store Password)"))
	FString KeyPassword;

	// This is the file that keytool outputs, specified with the -keystore parameter (file should be in <Project>/Build/Android)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = DebugSigning, Meta = (DisplayName = "Debug Key Store (output of keytool, placed in <Project>/Build/Android)"))
	FString DebugKeyStore;
	
	// This is the name of the key that you specified with the -alias parameter to keytool
	UPROPERTY(GlobalConfig, EditAnywhere, Category = DebugSigning, Meta = (DisplayName = "Debug Key Alias (-alias parameter to keytool)"))
	FString DebugKeyAlias;
	
	// This is the password that you specified FOR THE KEYSTORE NOT THE KEY, when running keytool (either with -storepass or by typing it in).
	UPROPERTY(GlobalConfig, EditAnywhere, Category = DebugSigning, Meta = (DisplayName = "Debug Key Store Password (-storepass parameter to keytool)"))
	FString DebugKeyStorePassword;
	
	// This is the password for the key that you may have specified with keytool, if it's different from the keystore password. Leave blank to use same as Debug Keystore
	UPROPERTY(GlobalConfig, EditAnywhere, Category = DebugSigning, Meta = (DisplayName = "Debug Key Password (leave blank to use Debug Key Store Password)"))
	FString DebugKeyPassword;

	// Enable Arm64 support?
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Support arm64 [aka arm64-v8a]"))
	bool bBuildForArm64;

	// Enable x86-64 support? [CURRENTLY FOR FULL SOURCE GAMES ONLY]
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Support x86_64 [aka x64]"))
	bool bBuildForX8664;

	// Include shaders for devices supporting OpenGL ES 3.2 and above (default)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Support OpenGL ES3.2"))
	bool bBuildForES31;

	// Support the Vulkan RHI and include Vulkan shaders
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Support Vulkan"))
	bool bSupportsVulkan;

	// Enable Vulkan SM5 rendering support
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Support Vulkan Desktop [Experimental]"))
	bool bSupportsVulkanSM5;

	/** Directory for Debug Vulkan Layers to package */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Debug Vulkan Layer Directory"))
	FDirectoryPath DebugVulkanLayerDirectory;

	/** Debug Vulkan Device Layers to enable */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Debug Vulkan Device Layers"))
	TArray<FString> DebugVulkanDeviceLayers;

	/** Debug Vulkan Instance Layers to enable */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Debug Vulkan Instance Layers"))
	TArray<FString> DebugVulkanInstanceLayers;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Support Backbuffer Sampling on OpenGL",
	ToolTip = "Whether to render to an offscreen surface instead of render to backbuffer directly on android opengl platform. Enable it if you'd like to support UMG background blur on android opengl."))
	bool bAndroidOpenGLSupportsBackbufferSampling;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Force Emulator profile selection in non-shipping builds"))
	bool bForceEmulatorProfileSelectionInNonShippingBuilds;
	
	// Whether to detect Vulkan device support by default, if the project is packaged with Vulkan support. If unchecked, the -detectvulkan commandline will enable Vulkan detection.
	UPROPERTY(GlobalConfig, EditAnywhere, AdvancedDisplay, Category = Build, meta = (DisplayName = "Detect Vulkan device support", EditCondition = "bSupportsVulkan"))
	bool bDetectVulkanByDefault;

	// Build the shipping config with hidden visibility by default. Results in smaller .so file but will also removes symbols used to display callstack dumps.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = AdvancedBuild, meta = (DisplayName = "Build with hidden symbol visibility in shipping config."))
	bool bBuildWithHiddenSymbolVisibility;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = AdvancedBuild, meta = (DisplayName = "Enable compression of debug symbols"))
	bool bCompressDebugSymbols;
	
	// Disables extra checks for buffer overflows, comes with perf improvement, but might make tracing stack corruptions in production harder. Note that _FORTIFY_SOURCE=2 is still enabled by the toolchain providing lightweight stack checks
	UPROPERTY(GlobalConfig, EditAnywhere, Category = AdvancedBuild, meta = (DisplayName = "Disable extra checks for buffer overflows"))
	bool bDisableStackProtector;

	// Disable libc++_shared dependency validation in all .so files linked with libUnreal.so
	UPROPERTY(GlobalConfig, EditAnywhere, Category = AdvancedBuild, meta = (DisplayName = "Disable libc++_shared dependency validation in all dependencies"))
	bool bDisableLibCppSharedDependencyValidation;

	// Always save .so file with symbols allowing use of addr2line on raw callstack addresses.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = AdvancedBuild, meta = (DisplayName = "Always save a copy of the libUnreal.so with symbols. [Experimental]"))
	bool bSaveSymbols;

	// Strip shader reflection information under Android to avoid issues on older drivers
	UPROPERTY(config, EditAnywhere, Category = AdvancedBuild, meta = (
	DisplayName = "Strip shader reflection information",
	ToolTip = "If true, strip shader reflection information under Android",
	ConfigRestartRequired = true))
	bool bStripShaderReflection;

	// If selected, the checked architectures will be split into separate .apk files [CURRENTLY FOR FULL SOURCE GAMES ONLY]
	// @todo android fat binary: Currently, there isn't much utility in merging multiple .so's into a single .apk except for debugging,
	// but we can't properly handle multiple GPU architectures in a single .apk, so we are disabling the feature for now
	// The user will need to manually select the apk to run in their Visual Studio debugger settings (see Override APK in TADP, for instance)
// 	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build)
// 	bool bSplitIntoSeparateApks;

	// Should Google Play support be enabled?
	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices)
	bool bEnableGooglePlaySupport;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices, meta = (DisplayName = "Request code for showing GooglePlay Games activities"), meta = (ToolTip = "This request code is reported in onActivityResult when Achievements or Leaderboards UIs are closed"))
	int RequestCodeForPlayGamesActivities;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices, meta = (DisplayName = "GooglePlay Games Oauth Client ID registered for game server"), meta = (ToolTip = "No auth code will be requested on login if empty. Setting up Server-side API Access (see https://developers.google.com/games/services/android/offline-access)"))
	FString PlayGamesClientId;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices, meta = (DisplayName = "Enable GooglePlay Games offline access"), meta = (ToolTip = "Include refresh token when Auth code is exchanged. Setting up Server-side API Access (see https://developers.google.com/games/services/android/offline-access)"))
	bool bForceRefreshToken;

	// The app id obtained from the Google Play Developer Console
	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices)
	FString GamesAppID;

	// Mapping of game achievement names to IDs generated by Google Play.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices)
	TArray<FGooglePlayAchievementMapping> AchievementMap;

	// Mapping of game leaderboard names to IDs generated by Google Play.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices)
	TArray<FGooglePlayLeaderboardMapping> LeaderboardMap;

	// Enabling this includes the AdMob SDK and will be detected by Google Play Console on upload of APK.  Disable if you do not need ads to remove this warning.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices, meta = (DisplayName = "Include AdMob support for ads"))
	bool bSupportAdMob;

	// AdMob Application ID
	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices, meta = (DisplayName = "AdMob App ID"), meta = (EditCondition = "bSupportAdMob"), meta = (ToolTip = "AdMob Application ID (see https://support.google.com/admob/answer/7356431)"))
	FString AdMobAppID;

	// Admob TagForChildDirectedTreatment (see https://developers.google.com/admob/android/targeting)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices, meta = (DisplayName = "AdMob Tag For Child-Directed Treatment"), meta = (EditCondition = "bSupportAdMob"), meta = (ToolTip = "Admob TagForChildDirectedTreatment (see https://developers.google.com/admob/android/targeting)"))
	TEnumAsByte<ETagForChildDirectedTreatment::Type> TagForChildDirectedTreatment;

	// Admob TagForUnderAgeOfConsent (see https://developers.google.com/admob/android/targeting)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices, meta = (DisplayName = "AdMob Tag For Under Age Of Consent"), meta = (EditCondition = "bSupportAdMob"), meta = (ToolTip = "Admob TagForUnderAgeOfConsent (see https://developers.google.com/admob/android/targeting)"))
	TEnumAsByte<ETagForUnderAgeOfConsent::Type> TagForUnderAgeOfConsent;

	// Admob MaxAdContentRating (see https://developers.google.com/admob/android/targeting)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices, meta = (DisplayName = "AdMob Max ad content rating"), meta = (EditCondition = "bSupportAdMob"), meta = (ToolTip = "Admob MaxAdContentRating (see https://developers.google.com/admob/android/targeting)"))
	TEnumAsByte<EMaxAdContentRating::Type> MaxAdContentRating;

	// The unique identifier for the ad obtained from AdMob.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices)
	FString AdMobAdUnitID;

	// Identifiers for ads obtained from AdMob
	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices)
	TArray<FString> AdMobAdUnitIDs;

	// The unique identifier for this application (needed for IAP)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices)
	FString GooglePlayLicenseKey;

	// The sender id obtained from Firebase Console, leave blank to disable (associate this with your app in Google Player Developer Console).
	UPROPERTY(GlobalConfig, EditAnywhere, Category = GooglePlayServices, meta = (DisplayName = "Google Cloud Messaging Sender ID"))
	FString GCMClientSenderID;

	/** Show the launch image as a startup slash screen */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = LaunchImages, meta = (DisplayName = "Show launch image"))
	bool bShowLaunchImage;

	/** Allows accelerometer, magnetometer, and gyroscope event handling, disabling may improve performance. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Input, meta = (DisplayName = "Allow IMU Sampling"))
	bool bAllowIMU;

	// If checked, Bluetooth connected controllers will send input
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Input, meta = (DisplayName = "Allow Bluetooth controllers"))
	bool bAllowControllers;

	// If checked, controllers will not send Android_Back and Android_Menu events that might cause unnecce
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Input, meta = (DisplayName = "Block Android system keys being sent from controllers"))
	bool bBlockAndroidKeysOnControllers;

	// Block force feedback on the device when controllers are attached.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Input, meta = (DisplayName = "Block force feedback on the device when controllers are attached"))
	bool bControllersBlockDeviceFeedback;

	/** Sample rate to run the audio mixer with. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", Meta = (DisplayName = "Audio Mixer Sample Rate"))
	int32 AudioSampleRate;

	/** The amount of audio to compute each callback block. Lower values decrease latency but may increase CPU cost. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "512", ClampMax = "4096", DisplayName = "Callback Buffer Size"))
	int32 AudioCallbackBufferFrameSize;

	/** The number of buffers to keep enqueued. More buffers increases latency, but can compensate for variable compute availability in audio callbacks on some platforms. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "2", UIMin = "2", DisplayName = "Number of Buffers To Enqueue"))
	int32 AudioNumBuffersToEnqueue;

	/** The max number of channels (voices) to limit for this platform. The max channels used will be the minimum of this value and the global audio quality settings. A value of 0 will not apply a platform channel count max. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "0", UIMin = "0", DisplayName = "Max Channels"))
	int32 AudioMaxChannels;

	/** The number of workers to use to compute source audio. Will only use up to the max number of sources. Will evenly divide sources to each source worker. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "0", UIMin = "0", DisplayName = "Number of Source Workers"))
	int32 AudioNumSourceWorkers;

	/** Which of the currently enabled spatialization plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString SpatializationPlugin;

	/** Which of the currently enabled source data override plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString SourceDataOverridePlugin;

	/** Which of the currently enabled reverb plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString ReverbPlugin;

	/** Which of the currently enabled occlusion plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString OcclusionPlugin;

	/** Various overrides for how this platform should handle compression and decompression */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FPlatformRuntimeAudioCompressionOverrides CompressionOverrides;

	/** This determines the max amount of memory that should be used for the cache at any given time. If set low (<= 8 MB), it lowers the size of individual chunks of audio during cook. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides|Stream Caching", meta = (DisplayName = "Max Cache Size (KB)"))
	int32 CacheSizeKB;

	/** This overrides the default max chunk size used when chunking audio for stream caching (ignored if < 0) */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides|Stream Caching", meta = (DisplayName = "Max Chunk Size Override (KB)"))
	int32 MaxChunkSizeOverrideKB;

	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides")
	bool bResampleForDevice;

	/** Quality Level to COOK SoundCues at (if set, all other levels will be stripped by the cooker). */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides", meta = (DisplayName = "Sound Cue Cook Quality"))
	int32 SoundCueCookQualityIndex = INDEX_NONE;

	// Mapping of which sample rates are used for each sample rate quality for a specific platform.

	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "Max"))
	float MaxSampleRate;

	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "High"))
	float HighSampleRate;

	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "Medium"))
	float MedSampleRate;

	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "Low"))
	float LowSampleRate;

	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "Min"))
	float MinSampleRate;

	// Scales all compression qualities when cooking to this platform. For example, 0.5 will halve all compression qualities, and 1.0 will leave them unchanged.
	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides")
	float CompressionQualityModifier;

	// When set to anything beyond 0, this will ensure any SoundWaves longer than this value, in seconds, to stream directly off of the disk.
	UPROPERTY(GlobalConfig)
	float AutoStreamingThreshold;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** DEPRECATED 5.8 */ 
	UE_DEPRECATED(5.8, "This option has been removed, please don't use")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This option has been removed, please don't use"))
	TEnumAsByte<EAndroidGraphicsDebugger::Type> AndroidGraphicsDebugger;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** DEPRECATED 5.8 */
	UE_DEPRECATED(5.8, "This option has been removed, please don't use")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This option has been removed, please don't use"))
	FDirectoryPath MaliGraphicsDebuggerPath;

	// If checked, this will disable the security.perf_harden flag on the Android device when launching or installing your app via the generated batch file.
	// Disabling security.perf_harden is necessary for HWCPipe to be able to report performance counters on Mali devices.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = GraphicsDebugger)
	bool bEnableMaliPerfCounters;

	/** Include ETC2 textures when packaging with the Android (Multi) variant. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = MultiTextureFormats, meta = (DisplayName = "Include ETC2 textures"))
	bool bMultiTargetFormat_ETC2;

	/** Include DXT textures when packaging with the Android (Multi) variant. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = MultiTextureFormats, meta = (DisplayName = "Include DXT textures"))
	bool bMultiTargetFormat_DXT;

	/** Include ASTC textures when packaging with the Android (Multi) variant. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = MultiTextureFormats, meta = (DisplayName = "Include ASTC textures"))
	bool bMultiTargetFormat_ASTC;

	/** Priority for the ETC2 texture format when launching on device or packaging using Android_Multi. The highest priority format supported by the device will be used. Default value is 0.2. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = TextureFormatPriorities, meta = (DisplayName = "ETC2 texture format priority"))
	float TextureFormatPriority_ETC2;

	/** Priority for the DXT texture format when launching on device or packaging using Android_Multi. The highest priority format supported by the device will be used. Default value is 0.6. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = TextureFormatPriorities, meta = (DisplayName = "DXT texture format priority"))
	float TextureFormatPriority_DXT;

	/** Priority for the ASTC texture format when launching on device or packaging using Android_Multi. The highest priority format supported by the device will be used. Default value is 0.9. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = TextureFormatPriorities, meta = (DisplayName = "ASTC texture format priority"))
	float TextureFormatPriority_ASTC;

	/** If enabled and using the ARM ASTC encoder this will cause the ASTC encoder to optimize the encode to LZ compression in the same manner as Oodle. 
	Refer to the texture encoding settings (e.g. Final Uses RDO) for details on how this works and its benefits.
	*/
	UPROPERTY(Config, EditAnywhere, Category = MultiTextureFormats, meta = (DisplayName = "Enable ASTC RDO", ConfigHierarchyEditable))
	bool bASTCUseRDO; // This name must match for all ASTC platforms so it's the same string in the ini

	// Which SDK to package and compile Java with (a specific version or (without quotes) 'latest' for latest version on disk, or 'matchndk' to match the NDK API Level)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Project SDK Override", Meta = (DisplayName = "SDK API Level (specific version, 'latest', or 'matchndk' - see tooltip)"))
	FString SDKAPILevelOverride;

	// Which NDK to compile with (a specific version or (without quotes) 'latest' for latest version on disk). Note that choosing android-21 or later won't run on pre-5.0 devices.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Project SDK Override", Meta = (DisplayName = "NDK API Level (specific version or 'latest' - see tooltip)"))
	FString NDKAPILevelOverride;

	// Which build-tools to package with (a specific version or (without quotes) 'latest' for latest version on disk).
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Project SDK Override", Meta = (DisplayName = "Build-Tools Version (specific version or 'latest')"))
	FString BuildToolsOverride;

	/** Whether to enable LOD streaming for landscape visual meshes. Only supported on feature level Mobile or above. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Misc", Meta = (DisplayName = "Stream landscape visual mesh LODs"))
	bool bStreamLandscapeMeshLODs;

	// Enables WebViews to use DOM storage API
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Misc", Meta = (DisplayName = "Enable DOM storage for WebViews"))
	bool bEnableDomStorage;

	UE_API virtual void PostReloadConfig(class FProperty* PropertyThatWasLoaded) override;


#if WITH_EDITOR
	/** Called whenever a registered Android property changes. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FPropertyChanged, struct FPropertyChangedEvent&);
	FPropertyChanged OnPropertyChanged;

private:
	// UObject interface
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostInitProperties() override;
	UE_API void HandlesRGBHWSupport();
	UE_DEPRECATED(5.8, "HandleMetaQuestSupport deprecated and replaced by HandleXRSupport()")
	UE_API void HandleMetaQuestSupport() { HandleXRSupport(); }
	UE_API void HandleXRSupport();
	UE_API void RemoveExtraApplicationTag(FString TagToRemove);
	
	// End of UObject interface
	UE_API void EnsureValidGPUArch();
#endif
};

#undef UE_API
