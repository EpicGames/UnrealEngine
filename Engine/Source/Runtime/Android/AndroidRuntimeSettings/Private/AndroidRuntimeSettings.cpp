// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidRuntimeSettings.h"
#include "Modules/ModuleManager.h"
#include "UObject/UnrealType.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "HAL/IConsoleManager.h"
#include "Engine/RendererSettings.h"
#include "HAL/PlatformApplicationMisc.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AndroidRuntimeSettings)

#if WITH_EDITOR
#include "IAndroidTargetPlatformModule.h"
#include "IAndroidTargetPlatformControlsModule.h"
#endif

DEFINE_LOG_CATEGORY(LogAndroidRuntimeSettings);

UAndroidRuntimeSettings::UAndroidRuntimeSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Orientation(EAndroidScreenOrientation::Landscape)
	, MinAspectRatio(0.0f)
	, MaxAspectRatio(2.4f)
	, bAndroidVoiceEnabled(false)
	, bPackageForOpenXRImmersive(false)
	, bPackageForMetaQuest(false)
	, bEnableGooglePlaySupport(false)
	, RequestCodeForPlayGamesActivities(80002)
	, bForceRefreshToken(false)
	, bSupportAdMob(true)
	, bBlockAndroidKeysOnControllers(false)
	, AudioSampleRate(44100)
	, AudioCallbackBufferFrameSize(1024)
	, AudioNumBuffersToEnqueue(4)
	, CacheSizeKB(65536)
	, MaxSampleRate(48000)
	, HighSampleRate(32000)
    , MedSampleRate(24000)
    , LowSampleRate(12000)
	, MinSampleRate(8000)
	, CompressionQualityModifier(1)
	, bMultiTargetFormat_ETC2(true)
	, bMultiTargetFormat_DXT(true)
	, bMultiTargetFormat_ASTC(true)
	, TextureFormatPriority_ETC2(0.2f)
	, TextureFormatPriority_DXT(0.6f)
	, TextureFormatPriority_ASTC(0.9f)
	, bASTCUseRDO(false)
	, bStreamLandscapeMeshLODs(false)
{
	bBuildForES31 = bBuildForES31 || !bSupportsVulkan;
}

void UAndroidRuntimeSettings::PostReloadConfig(FProperty* PropertyThatWasLoaded)
{
	Super::PostReloadConfig(PropertyThatWasLoaded);

#if PLATFORM_ANDROID

	FPlatformApplicationMisc::SetGamepadsAllowed(bAllowControllers);

#endif //PLATFORM_ANDROID
}

#if WITH_EDITOR

void UAndroidRuntimeSettings::HandlesRGBHWSupport()
{
	const bool SupportssRGB = bPackageForMetaQuest || bPackageForOpenXRImmersive;
	URendererSettings* const Settings = GetMutableDefault<URendererSettings>();
	static auto* MobileUseHWsRGBEncodingCVAR = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.UseHWsRGBEncoding"));

	if (SupportssRGB != Settings->bMobileUseHWsRGBEncoding)
	{
		Settings->bMobileUseHWsRGBEncoding = SupportssRGB;
		Settings->UpdateSinglePropertyInConfigFile(Settings->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URendererSettings, bMobileUseHWsRGBEncoding)), GetDefaultConfigFilename());
	}

	if (MobileUseHWsRGBEncodingCVAR && MobileUseHWsRGBEncodingCVAR->GetInt() != (int)SupportssRGB)
	{
		MobileUseHWsRGBEncodingCVAR->Set((int)SupportssRGB);
	}

}

void UAndroidRuntimeSettings::HandleXRSupport()
{
	const FString SupportedDevicesTag("<meta-data android:name=\"com.oculus.supportedDevices\"");

	if (bPackageForMetaQuest)
	{
		if (!bBuildForArm64)
		{
			bBuildForArm64 = true;
			UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, bBuildForArm64)), GetDefaultConfigFilename());
			UE_LOGF(LogAndroidRuntimeSettings, Warning, "Enabled Package for Meta Quest devices. Support arm64 has been changed to true.\n");
		}

		int32 SupportedDevicesTagIndex = ExtraApplicationSettings.Find("com.oculus.supportedDevices");
		FString SupportedDevicesValue("quest2|questpro|quest3|quest3s");
		int32 SupportedDevicesIndex = ExtraApplicationSettings.Find(SupportedDevicesValue);

		// The supported devices tag is present but not up to date and does not contain all the currently supported devices.
		bool bNeedtoUpdateDevices = (SupportedDevicesTagIndex != INDEX_NONE) && (SupportedDevicesIndex == INDEX_NONE);

		// Remove the current supported devices value so that it can be added again with the updated value further down.
		if (bNeedtoUpdateDevices)
		{
			RemoveExtraApplicationTag(SupportedDevicesTag);
		}

		if (SupportedDevicesTagIndex == INDEX_NONE || bNeedtoUpdateDevices)
		{
			ExtraApplicationSettings.Append("<meta-data android:name=\"com.oculus.supportedDevices\" android:value=\"" + SupportedDevicesValue + "\" />");
			UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, ExtraApplicationSettings)), GetDefaultConfigFilename());
		}
	}
	else
	{
		// Clean up the supported devices metadata tag so it doesn't end up in the manifest when bPackageForMetaQuest is turned off.
		RemoveExtraApplicationTag(SupportedDevicesTag);
	}
}

void UAndroidRuntimeSettings::RemoveExtraApplicationTag(FString TagToRemove)
{
	int32 StartIndex = ExtraApplicationSettings.Find(TagToRemove);
	if (StartIndex != INDEX_NONE)
	{
		int32 EndIndex = ExtraApplicationSettings.Find(">", ESearchCase::IgnoreCase, ESearchDir::FromStart, StartIndex);
		if (EndIndex == INDEX_NONE)
		{
			UE_LOGF(LogAndroidRuntimeSettings, Warning, "RemoveExtraApplicationTag found what looks like corrupt xml, cannot remove tag.\n");
		}
		else
		{
			ExtraApplicationSettings.RemoveAt(StartIndex, EndIndex - StartIndex + 1);
			UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, ExtraApplicationSettings)), GetDefaultConfigFilename());
		}
	}
}

static void InvalidateAllAndroidPlatforms()
{
	ITargetPlatformModule* Module = FModuleManager::GetModulePtr<IAndroidTargetPlatformModule>("AndroidTargetPlatform");

	// call the delegate for each TP object
	for (ITargetPlatform* TargetPlatform : Module->GetTargetPlatforms())
	{
		FCoreDelegates::OnTargetPlatformChangedSupportedFormats.Broadcast(TargetPlatform);
	}
}

void UAndroidRuntimeSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Ensure that at least one architecture is supported
	if (!bBuildForX8664 && !bBuildForArm64)
	{
		bBuildForArm64 = true;
		UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, bBuildForArm64)), GetDefaultConfigFilename());
	}

	if (PropertyChangedEvent.Property != nullptr)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, bSupportsVulkan) ||
			PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, bBuildForES31))
		{
			// Supported shader formats changed so invalidate cache
			InvalidateAllAndroidPlatforms();

			OnPropertyChanged.Broadcast(PropertyChangedEvent);
		}
	}

	EnsureValidGPUArch();

	if (PropertyChangedEvent.Property != nullptr && PropertyChangedEvent.Property->GetName().StartsWith(TEXT("bMultiTargetFormat")))
	{
		UpdateSinglePropertyInConfigFile(PropertyChangedEvent.Property, GetDefaultConfigFilename());

		// Ensure we have at least one format for Android_Multi
		if (!bMultiTargetFormat_ETC2 && !bMultiTargetFormat_DXT && !bMultiTargetFormat_ASTC)
		{
			bMultiTargetFormat_ETC2 = true;
			UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, bMultiTargetFormat_ETC2)), GetDefaultConfigFilename());
		}

		// Notify the AndroidTargetPlatform module if it's loaded
		IAndroidTargetPlatformControlsModule* Module = FModuleManager::GetModulePtr<IAndroidTargetPlatformControlsModule>("AndroidTargetPlatformControls");
		if (Module)
		{
			Module->NotifyMultiSelectedFormatsChanged();
		}
	}

	if (PropertyChangedEvent.Property != nullptr && PropertyChangedEvent.Property->GetName().StartsWith(TEXT("TextureFormatPriority")))
	{
		UpdateSinglePropertyInConfigFile(PropertyChangedEvent.Property, GetDefaultConfigFilename());

		// Notify the AndroidTargetPlatform module if it's loaded
		IAndroidTargetPlatformControlsModule* Module = FModuleManager::GetModulePtr<IAndroidTargetPlatformControlsModule>("AndroidTargetPlatformControls");
		if (Module)
		{
			Module->NotifyMultiSelectedFormatsChanged();
		}
	}

	if (PropertyChangedEvent.Property != nullptr 
		&& (PropertyChangedEvent.Property->GetName().StartsWith(TEXT("bPackageForMetaQuest")) || PropertyChangedEvent.Property->GetName().StartsWith(TEXT("bPackageForOpenXRImmersive"))))
	{
		HandleXRSupport();
	}

	HandlesRGBHWSupport();
}

void UAndroidRuntimeSettings::PostInitProperties()
{
	Super::PostInitProperties();

	// If the config has an AdMobAdUnitID then we migrate it on load and clear the value
	if (!AdMobAdUnitID.IsEmpty())
	{
		AdMobAdUnitIDs.Add(AdMobAdUnitID);
		AdMobAdUnitID.Empty();
		TryUpdateDefaultConfigFile();
	}

	EnsureValidGPUArch();
	HandlesRGBHWSupport();
	HandleXRSupport();
}

void UAndroidRuntimeSettings::EnsureValidGPUArch()
{
	// Ensure that at least one GPU architecture is supported
	if (!bSupportsVulkan && !bBuildForES31 && !bSupportsVulkanSM5)
	{
		UE_LOGF(LogAndroidRuntimeSettings, Warning, "No GPU architecture is selected.\n");
		// Default to Vulkan for XR devices
		if (bPackageForMetaQuest || bPackageForOpenXRImmersive)
		{
			bSupportsVulkan = true;
			UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, bSupportsVulkan)), GetDefaultConfigFilename());
			UE_LOGF(LogAndroidRuntimeSettings, Warning, "Support Vulkan has been changed to true.\n");
		}
		else
		{
			bBuildForES31 = true;
			UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, bBuildForES31)), GetDefaultConfigFilename());
			UE_LOGF(LogAndroidRuntimeSettings, Warning, "Support OpenGL ES3.2 has been changed to true.\n");
		}

		// Supported shader formats changed so invalidate cache
		InvalidateAllAndroidPlatforms();
	}
}
#endif

