// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioMixerAndroid : ModuleRules
{
	public AudioMixerAndroid(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("TargetPlatform");
		PublicIncludePaths.Add("Runtime/AudioMixer/Public");
		PrivateIncludePaths.Add("Runtime/AudioMixer/Private");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"AudioMixerCore",
			}
			);
			
		if(Target.bCompileAgainstEngine)
        {
            AddEngineThirdPartyPrivateStaticDependencies(Target,
            "UEOgg",
            "Vorbis",
            "VorbisFile"
            );

            PrivateDependencyModuleNames.AddRange(
            new string[] {
                    "Engine"
                }
            );

            PublicDefinitions.Add("WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG=0");
        }
	}
}
