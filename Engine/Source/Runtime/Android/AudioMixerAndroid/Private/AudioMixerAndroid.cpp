// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixer.h"
#include "AudioMixerPlatformAndroid.h"
#include "Modules/ModuleManager.h"

class FAudioMixerModuleAndroid : public IAudioDeviceModule
{
public:
	
	virtual void StartupModule() override
	{
		IAudioDeviceModule::StartupModule();

		FModuleManager::Get().LoadModuleChecked(TEXT("AudioMixerCore"));
	}

	virtual bool IsAudioMixerModule() const override { return true; }

	virtual Audio::IAudioMixerPlatformInterface* CreateAudioMixerPlatformInterface() override
	{
		return new Audio::FMixerPlatformAndroid();
	}
};

IMPLEMENT_MODULE(FAudioMixerModuleAndroid, AudioMixerAndroid);
