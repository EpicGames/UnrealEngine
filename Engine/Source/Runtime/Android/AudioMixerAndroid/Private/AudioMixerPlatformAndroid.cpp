// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerPlatformAndroid.h"
#include "Modules/ModuleManager.h"
#include "AudioMixer.h"
#include "CoreGlobals.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"
#include "AudioDevice.h"

#if WITH_ENGINE
#include "AudioPluginUtilities.h"
#endif


#include <SLES/OpenSLES.h>
#include "SLES/OpenSLES_Android.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAudioMixerAndroid, Log, All);
DEFINE_LOG_CATEGORY(LogAudioMixerAndroid);

#define UNREAL_AUDIO_TEST_WHITE_NOISE 0

// Macro to check result for XAudio2 failure, get string version, log, and return false
#define OPENSLES_RETURN_ON_FAIL(Result)						\
	if (Result != SL_RESULT_SUCCESS)						\
	{														\
		const TCHAR* ErrorString = GetErrorString(Result);	\
		AUDIO_PLATFORM_ERROR(ErrorString);					\
		return false;										\
	}

#define OPENSLES_CHECK_ON_FAIL(Result)						\
	if (Result != SL_RESULT_SUCCESS)						\
	{														\
		const TCHAR* ErrorString = GetErrorString(Result);	\
		AUDIO_PLATFORM_ERROR(ErrorString);					\
		check(false);										\
	}

#define OPENSLES_LOG_ON_FAIL(Result)						\
	if (Result != SL_RESULT_SUCCESS)						\
	{														\
		const TCHAR* ErrorString = GetErrorString(Result);	\
		AUDIO_PLATFORM_ERROR(ErrorString);					\
	}

#if USE_ANDROID_JNI
extern int32 AndroidThunkCpp_GetMetaDataInt(const FString& Key);
#endif

namespace Audio
{
	FMixerPlatformAndroid::FMixerPlatformAndroid()
		: bSuspended(false)
		, bInitialized(false)
		, bInCallback(false)
		, NumSamplesPerRenderCallback(0)
		, NumSamplesPerDeviceCallback(0)
	{
	}

	FMixerPlatformAndroid::~FMixerPlatformAndroid()
	{
		if (bInitialized)
		{
			TeardownHardware();
		}
	}

	const TCHAR* FMixerPlatformAndroid::GetErrorString(SLresult Result)
	{
		switch (Result)
		{
			case SL_RESULT_PRECONDITIONS_VIOLATED:	return TEXT("SL_RESULT_PRECONDITIONS_VIOLATED");
			case SL_RESULT_PARAMETER_INVALID:		return TEXT("SL_RESULT_PARAMETER_INVALID");
			case SL_RESULT_MEMORY_FAILURE:			return TEXT("SL_RESULT_MEMORY_FAILURE");
			case SL_RESULT_RESOURCE_ERROR:			return TEXT("SL_RESULT_RESOURCE_ERROR");
			case SL_RESULT_RESOURCE_LOST:			return TEXT("SL_RESULT_RESOURCE_LOST");
			case SL_RESULT_IO_ERROR:				return TEXT("SL_RESULT_IO_ERROR");
			case SL_RESULT_BUFFER_INSUFFICIENT:		return TEXT("SL_RESULT_BUFFER_INSUFFICIENT");
			case SL_RESULT_CONTENT_CORRUPTED:		return TEXT("SL_RESULT_CONTENT_CORRUPTED");
			case SL_RESULT_CONTENT_UNSUPPORTED:		return TEXT("SL_RESULT_CONTENT_UNSUPPORTED");
			case SL_RESULT_CONTENT_NOT_FOUND:		return TEXT("SL_RESULT_CONTENT_NOT_FOUND");
			case SL_RESULT_PERMISSION_DENIED:		return TEXT("SL_RESULT_PERMISSION_DENIED");
			case SL_RESULT_FEATURE_UNSUPPORTED:		return TEXT("SL_RESULT_FEATURE_UNSUPPORTED");
			case SL_RESULT_INTERNAL_ERROR:			return TEXT("SL_RESULT_INTERNAL_ERROR");
			case SL_RESULT_OPERATION_ABORTED:		return TEXT("SL_RESULT_OPERATION_ABORTED");
			case SL_RESULT_CONTROL_LOST:			return TEXT("SL_RESULT_CONTROL_LOST");

			default:
			case SL_RESULT_UNKNOWN_ERROR:			return TEXT("SL_RESULT_UNKNOWN_ERROR");
		}
	}

	int32 FMixerPlatformAndroid::GetDeviceBufferSize(int32 RenderCallbackSize) const
	{
#if USE_ANDROID_JNI
		// Override with platform-specific frames per buffer size
		int32 MinFramesPerBuffer = AndroidThunkCpp_GetMetaDataInt(TEXT("audiomanager.framesPerBuffer"));

		int32 BufferSizeToUse = MinFramesPerBuffer;
		while (BufferSizeToUse < RenderCallbackSize)
		{
			BufferSizeToUse += MinFramesPerBuffer;
		}

		return BufferSizeToUse;
#else
		ensureMsgf(false, TEXT("JNI not supported on this platform. Audio output may be broken."));
		return 1024;
#endif
	}

	bool FMixerPlatformAndroid::InitializeHardware()
	{
		if (bInitialized)
		{
			return false;
		}

		SLresult Result;
		SLEngineOption EngineOption[] = { {(SLuint32) SL_ENGINEOPTION_THREADSAFE, (SLuint32) SL_BOOLEAN_TRUE} };

		// Create engine
		Result = slCreateEngine( &SL_EngineObject, 1, EngineOption, 0, NULL, NULL);
		OPENSLES_CHECK_ON_FAIL(Result);

		// Realize the engine
		Result = (*SL_EngineObject)->Realize(SL_EngineObject, SL_BOOLEAN_FALSE);
		OPENSLES_CHECK_ON_FAIL(Result);

		// get the engine interface, which is needed in order to create other objects
		Result = (*SL_EngineObject)->GetInterface(SL_EngineObject, SL_IID_ENGINE, &SL_EngineEngine);
		OPENSLES_CHECK_ON_FAIL(Result);

		// create output mix
		Result = (*SL_EngineEngine)->CreateOutputMix(SL_EngineEngine, &SL_OutputMixObject, 0, NULL, NULL );
		OPENSLES_CHECK_ON_FAIL(Result);

		// realize the output mix
		Result = (*SL_OutputMixObject)->Realize(SL_OutputMixObject, SL_BOOLEAN_FALSE);
		OPENSLES_CHECK_ON_FAIL(Result);

		bInitialized = true;

		return true;
	}

	bool FMixerPlatformAndroid::TeardownHardware()
	{
		if(!bInitialized)
		{
			return true;
		}

		// Teardown OpenSLES..
		// Destroy the SLES objects in reverse order of creation:
		if (SL_OutputMixObject)
		{
			(*SL_OutputMixObject)->Destroy(SL_OutputMixObject);
			SL_OutputMixObject = nullptr;
		}

		if (SL_EngineObject)
		{
			(*SL_EngineObject)->Destroy(SL_EngineObject);

			SL_EngineObject = nullptr;
			SL_EngineEngine = nullptr;
		}

		bInitialized = false;

		return true;
	}

	bool FMixerPlatformAndroid::IsInitialized() const
	{
		return bInitialized;
	}

	bool FMixerPlatformAndroid::GetNumOutputDevices(uint32& OutNumOutputDevices)
	{
		OutNumOutputDevices = 1;
		return true;
	}

	bool FMixerPlatformAndroid::GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo)
	{
#if USE_ANDROID_JNI
		OutInfo.Name = TEXT("Android Audio Device");
		OutInfo.DeviceId = 0;
		OutInfo.bIsSystemDefault = true;
		OutInfo.SampleRate = AndroidThunkCpp_GetMetaDataInt(TEXT("audiomanager.optimalSampleRate"));
		OutInfo.NumChannels = 2; // Android doesn't support surround sound
		OutInfo.Format = EAudioMixerStreamDataFormat::Int16;
		OutInfo.OutputChannelArray.SetNum(2);
		OutInfo.OutputChannelArray[0] = EAudioMixerChannel::FrontLeft;
		OutInfo.OutputChannelArray[1] = EAudioMixerChannel::FrontRight;
		return true;
#else
		return false;
#endif
	}

	bool FMixerPlatformAndroid::GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const
	{
		OutDefaultDeviceIndex = 0;
		return true;
	}

	bool FMixerPlatformAndroid::OpenAudioStream(const FAudioMixerOpenStreamParams& Params)
	{
		if (!bInitialized || AudioStreamInfo.StreamState != EAudioOutputStreamState::Closed)
		{
			return false;
		}

		OpenStreamParams = Params;

		AudioStreamInfo.Reset();

		AudioStreamInfo.OutputDeviceIndex = 0;
		AudioStreamInfo.NumOutputFrames = OpenStreamParams.NumFrames;
		AudioStreamInfo.NumBuffers = FMath::Max(OpenStreamParams.NumBuffers, 4);
		AudioStreamInfo.AudioMixer = OpenStreamParams.AudioMixer;

		if (!GetOutputDeviceInfo(AudioStreamInfo.OutputDeviceIndex, AudioStreamInfo.DeviceInfo))
		{
			return false;
		}

		AudioStreamInfo.DeviceInfo.SampleRate = OpenStreamParams.SampleRate;

		SLresult Result;

		FAudioPlatformSettings PlatformSettings = GetPlatformSettings();

		// Set up circular buffer between our rendering buffer size and the device's buffer size.
		// Since we are only using this circular buffer on a single thread, we do not need to add extra slack.
		NumSamplesPerRenderCallback = OpenStreamParams.NumFrames * AudioStreamInfo.DeviceInfo.NumChannels;
		NumSamplesPerDeviceCallback = PlatformSettings.CallbackBufferFrameSize * AudioStreamInfo.DeviceInfo.NumChannels;
		const int32 MaxCircularBufferCapacity = FMath::Max<int32>(NumSamplesPerRenderCallback, NumSamplesPerDeviceCallback) * 2;
		CircularOutputBuffer.SetCapacity(MaxCircularBufferCapacity);

		DeviceBuffer.Reset();
		DeviceBuffer.AddUninitialized(NumSamplesPerDeviceCallback);

		// Data Info:
		SLDataLocator_AndroidSimpleBufferQueue LocationBuffer = { SL_DATALOCATOR_ANDROIDSIMPLEBUFFERQUEUE, 1};

		// PCM Info
		SLDataFormat_PCM PCM_Format = {
			SL_DATAFORMAT_PCM,
			(SLuint32)AudioStreamInfo.DeviceInfo.NumChannels,
			(SLuint32)(AudioStreamInfo.DeviceInfo.SampleRate * 1000), // NOTE: OpenSLES has sample rates specified in millihertz.
			SL_PCMSAMPLEFORMAT_FIXED_16,
			SL_PCMSAMPLEFORMAT_FIXED_16,
			SL_SPEAKER_FRONT_LEFT | SL_SPEAKER_FRONT_RIGHT,
			SL_BYTEORDER_LITTLEENDIAN
		};

		SLDataSource SoundDataSource = { &LocationBuffer, &PCM_Format };

		// configure audio sink
		SLDataLocator_OutputMix OutputMix = { SL_DATALOCATOR_OUTPUTMIX, SL_OutputMixObject };
		SLDataSink AudioSink = { &OutputMix, nullptr };

		// create audio player
		const SLInterfaceID	InterfaceIds[] = {SL_IID_BUFFERQUEUE, SL_IID_VOLUME};
		const SLboolean Req[] = {SL_BOOLEAN_TRUE, SL_BOOLEAN_TRUE};

		Result = (*SL_EngineEngine)->CreateAudioPlayer(SL_EngineEngine, &SL_PlayerObject, &SoundDataSource, &AudioSink, sizeof(InterfaceIds) / sizeof(SLInterfaceID), InterfaceIds, Req);
		OPENSLES_RETURN_ON_FAIL(Result);

		// realize the player
		Result = (*SL_PlayerObject)->Realize(SL_PlayerObject, SL_BOOLEAN_FALSE);
		OPENSLES_RETURN_ON_FAIL(Result);

		// get the play interface
		Result = (*SL_PlayerObject)->GetInterface(SL_PlayerObject, SL_IID_PLAY, &SL_PlayerPlayInterface);
		OPENSLES_RETURN_ON_FAIL(Result);

		// buffer system
		Result = (*SL_PlayerObject)->GetInterface(SL_PlayerObject, SL_IID_BUFFERQUEUE, &SL_PlayerBufferQueue);
		OPENSLES_RETURN_ON_FAIL(Result);

		Result = (*SL_PlayerBufferQueue)->RegisterCallback(SL_PlayerBufferQueue, OpenSLBufferQueueCallback, (void*)this);
		OPENSLES_RETURN_ON_FAIL(Result);

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Open;

		return true;
	}

	bool FMixerPlatformAndroid::CloseAudioStream()
	{
		if (!bInitialized || (AudioStreamInfo.StreamState != EAudioOutputStreamState::Open && AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped))
		{
			return false;
		}

		SLresult Result =(*SL_PlayerBufferQueue)->RegisterCallback(SL_PlayerBufferQueue, nullptr, nullptr);

		(*SL_PlayerObject)->Destroy(SL_PlayerObject);

		SL_PlayerObject = nullptr;
		SL_PlayerPlayInterface = nullptr;
		SL_PlayerBufferQueue = nullptr;

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Closed;

		return true;
	}

	bool FMixerPlatformAndroid::StartAudioStream()
	{
		BeginGeneratingAudio();

		// set the player's state to playing
		SLresult Result = (*SL_PlayerPlayInterface)->SetPlayState(SL_PlayerPlayInterface, SL_PLAYSTATE_PLAYING);
		OPENSLES_CHECK_ON_FAIL(Result);

		return true;
	}

	bool FMixerPlatformAndroid::StopAudioStream()
	{
		if(!bInitialized || AudioStreamInfo.StreamState != EAudioOutputStreamState::Running)
		{
			return false;
		}

		if (AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped)
		{
			// set the player's state to stopped
			SLresult Result = (*SL_PlayerPlayInterface)->SetPlayState(SL_PlayerPlayInterface, SL_PLAYSTATE_STOPPED);
			OPENSLES_CHECK_ON_FAIL(Result);

			if (AudioStreamInfo.StreamState == EAudioOutputStreamState::Running)
			{
				StopGeneratingAudio();
			}

			check(AudioStreamInfo.StreamState == EAudioOutputStreamState::Stopped);
		}

		return true;
	}

	FAudioPlatformDeviceInfo FMixerPlatformAndroid::GetPlatformDeviceInfo() const
	{
		return AudioStreamInfo.DeviceInfo;
	}

 	FAudioPlatformSettings FMixerPlatformAndroid::GetPlatformSettings() const
 	{
#if WITH_ENGINE
		FAudioPlatformSettings PlatformSettings = FAudioPlatformSettings::GetPlatformSettings(FPlatformProperties::GetRuntimeSettingsClassName());
#else
		FAudioPlatformSettings PlatformSettings = FAudioPlatformSettings();
#endif // WITH_ENGINE

		PlatformSettings.CallbackBufferFrameSize = GetDeviceBufferSize(PlatformSettings.CallbackBufferFrameSize);
		return PlatformSettings;
	}

	void FMixerPlatformAndroid::SuspendContext()
	{
		FScopeLock ScopeLock(&SuspendedCriticalSection);

		if (!bSuspended)
		{
			UE_LOGF(LogAudioMixerAndroid, Display, "Suspending android audio renderer");

			// set the player's state to paused
			SLresult result = (*SL_PlayerPlayInterface)->SetPlayState(SL_PlayerPlayInterface, SL_PLAYSTATE_PAUSED);
			check(SL_RESULT_SUCCESS == result);

			bSuspended = true;
		}
	}

	void FMixerPlatformAndroid::ResumeContext()
	{
		FScopeLock ScopeLock(&SuspendedCriticalSection);

		// set the player's state to paused
		if (bSuspended)
		{
			UE_LOGF(LogAudioMixerAndroid, Display, "Resuming android audio renderer");

			SLresult result = (*SL_PlayerPlayInterface)->SetPlayState(SL_PlayerPlayInterface, SL_PLAYSTATE_PLAYING);
			check(SL_RESULT_SUCCESS == result);

			bSuspended = false;
		}
	}

	void FMixerPlatformAndroid::SubmitBuffer(const uint8* Buffer)
	{
		check(DeviceBuffer.Num() == NumSamplesPerDeviceCallback);

		int32 PushResult = CircularOutputBuffer.Push((const int16*)Buffer, NumSamplesPerRenderCallback);
		check(PushResult == NumSamplesPerRenderCallback)

		while (CircularOutputBuffer.Num() >= NumSamplesPerDeviceCallback)
		{
			int32 PopResult = CircularOutputBuffer.Pop(DeviceBuffer.GetData(), NumSamplesPerDeviceCallback);
			check(PopResult == NumSamplesPerDeviceCallback);

			const auto BufferSize = NumSamplesPerDeviceCallback * sizeof(int16);
			SLresult Result = (*SL_PlayerBufferQueue)->Enqueue(SL_PlayerBufferQueue, Buffer, BufferSize);
			OPENSLES_LOG_ON_FAIL(Result);
		}
	}	

	FString FMixerPlatformAndroid::GetDefaultDeviceName()
	{
		return FString();
	}

	void FMixerPlatformAndroid::OpenSLBufferQueueCallback(SLAndroidSimpleBufferQueueItf InQueueInterface, void* pContext)
	{
		FMixerPlatformAndroid* MixerPlatformAndroid = (FMixerPlatformAndroid*)pContext;
		if (MixerPlatformAndroid != nullptr)
		{
			MixerPlatformAndroid->ReadNextBuffer();
		}
	}
}
