// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"
#include "DSP/Dsp.h"
#include <SLES/OpenSLES.h>
#include "SLES/OpenSLES_Android.h"

// Any platform defines
namespace Audio
{

	class FMixerPlatformAndroid : public IAudioMixerPlatformInterface
	{

	public:

		FMixerPlatformAndroid();
		~FMixerPlatformAndroid();

		//~ Begin IAudioMixerPlatformInterface
		virtual FString GetPlatformApi() const override { return TEXT("OpenSLES"); }
		virtual bool InitializeHardware() override;
		virtual bool TeardownHardware() override;
		virtual bool IsInitialized() const override;
		virtual bool GetNumOutputDevices(uint32& OutNumOutputDevices) override;
		virtual bool GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo) override;
		virtual bool GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const override;
		virtual bool OpenAudioStream(const FAudioMixerOpenStreamParams& Params) override;
		virtual bool CloseAudioStream() override;
		virtual bool StartAudioStream() override;
		virtual bool StopAudioStream() override;
		virtual FAudioPlatformDeviceInfo GetPlatformDeviceInfo() const override;
		virtual void SubmitBuffer(const uint8* Buffer) override;
		virtual FString GetDefaultDeviceName() override;
		virtual FAudioPlatformSettings GetPlatformSettings() const override;
		virtual void SuspendContext() override;
		virtual void ResumeContext() override;

		//~ End IAudioMixerPlatformInterface
		
	private:
		const TCHAR* GetErrorString(SLresult Result);

		int32 GetDeviceBufferSize(int32 RenderCallbackSize) const;

		SLObjectItf	SL_EngineObject;
		SLEngineItf	SL_EngineEngine;
		SLObjectItf	SL_OutputMixObject;
		SLObjectItf	SL_PlayerObject;
		SLPlayItf SL_PlayerPlayInterface;
		SLAndroidSimpleBufferQueueItf SL_PlayerBufferQueue;

		FCriticalSection SuspendedCriticalSection;

		bool bSuspended;
		bool bInitialized;
		bool bInCallback;
		
		// This buffer is pushed to and popped from in the SubmitBuffer callback. 
		// This is required for devices that require frame counts per callback that are not powers of two.
		Audio::TCircularAudioBuffer<int16> CircularOutputBuffer;

		// This is the buffer we pop CircularOutputBuffer into in SubmitBuffer.
		TArray<int16> DeviceBuffer;

		int32 NumSamplesPerRenderCallback;
		int32 NumSamplesPerDeviceCallback;

		static void OpenSLBufferQueueCallback( SLAndroidSimpleBufferQueueItf InQueueInterface, void* pContext );		
	};

}

